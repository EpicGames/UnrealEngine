// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/AutomationTest.h"
#include "TestDataAndFunctionNames.h"
#include "SlateTestBase.h"
#include "SWidgetMock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SWindow.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::SlateWidgetAutomationTest
{
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FFocusTest, "Slate.FocusTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

template<class T>
class FSlateTestFocus : public FSlateTestBase<T>
{
public:

	using FSlateTestBase<T>::GetSubjectWidget;
	using FSlateTestBase<T>::GetMainWindow;
	using FSlateTestBase<T>::CreateWidget;

	void Setup(FAutomationTestBase* TestObj) override
	{
		// 1. In the main window, insert a panel widget containing two widgets of the same type that support focus.
		// We call the two widgets SubjectWidget and SiblingWidget. SubjectWidget is the target of focus in the test.
		PanelWidget = SNew(SWidgetMock<SVerticalBox>);
		SiblingWidget = CreateWidget();

		PanelWidget->AddSlot()[GetSubjectWidget()];
		PanelWidget->AddSlot()[SiblingWidget.ToSharedRef()];

		GetMainWindow()->SetContent(PanelWidget.ToSharedRef());

		// 2. Initially set focus to the sibling widget. 
		FSlateApplication::Get().SetUserFocus(0, SiblingWidget);

		// 3. Erase any side effect that we might have caused as a result of setting up the test.
		GetSubjectWidget()->ClearFunctionRecords();
		SiblingWidget->ClearFunctionRecords();

	}

	void Run(FAutomationTestBase* TestObj) override
	{
		// Manually set the focus on the subject widget.
		FSlateApplication::Get().SetUserFocus(0, GetSubjectWidget());
	}

	void Validate(FAutomationTestBase* TestObj ) override
	{
		// Verify the expected callbacks were called on the right widgets.
		TestObj->AddErrorIfFalse(GetSubjectWidget()->GetValueInFunctionCalls(FTestFunctionNames::NAME_OnFocusReceived) == 1, FString::Printf(TEXT("In %s: OnFocusReceived callback was not called on the widget receiving focus."), GetGeneratedTypeName<T>()));
		TestObj->AddErrorIfFalse(GetSubjectWidget()->GetValueInFunctionCalls(FTestFunctionNames::NAME_OnFocusChanging) == 1, FString::Printf(TEXT("In %s: OnFocusChanging callback was not called on the widget receiving focus."), GetGeneratedTypeName<T>()));
		TestObj->AddErrorIfFalse(SiblingWidget->GetValueInFunctionCalls(FTestFunctionNames::NAME_OnFocusLost) == 1, FString::Printf(TEXT("In %s: OnFocusLost callback was not called on the widget initally having focus."), GetGeneratedTypeName<T>()));
		TestObj->AddErrorIfFalse(SiblingWidget->GetValueInFunctionCalls(FTestFunctionNames::NAME_OnFocusChanging) == 1, FString::Printf(TEXT("In %s: OnFocusChanging callback was not called on the widget initally having focus."), GetGeneratedTypeName<T>()));

		// Verify that unexpected callbacks were NOT called on the widgets.
		TestObj->AddErrorIfFalse(GetSubjectWidget()->GetValueInFunctionCalls(FTestFunctionNames::NAME_OnFocusLost) == 0, FString::Printf(TEXT("In %s: OnFocusLost callback was called the widget receiving focus."), GetGeneratedTypeName<T>()));
		TestObj->AddErrorIfFalse(SiblingWidget->GetValueInFunctionCalls(FTestFunctionNames::NAME_OnFocusReceived) == 0, FString::Printf(TEXT("In %s: OnFocusReceived callback was called on the widget initally having focus."), GetGeneratedTypeName<T>()));

		TestObj->AddErrorIfFalse(GetSubjectWidget()->HasAnyUserFocus().IsSet(), FString::Printf(TEXT("In %s: No user is focusing on the widget receiving focus."), GetGeneratedTypeName<T>()));
		TestObj->AddErrorIfFalse(GetMainWindow()->HasFocusedDescendants(), FString::Printf(TEXT("In %s: The parent window does not detect that its descendant has focus because it detects no user that is focusing on it."), GetGeneratedTypeName<T>()));
		TestObj->AddErrorIfFalse(PanelWidget->HasFocusedDescendants(), FString::Printf(TEXT("In %s: The immediate parent of the widget does not detect that its descendant has focus because it detects no user that is focusing on it."), GetGeneratedTypeName<T>()));
	}

	virtual void TearDown(FAutomationTestBase* TestObj)
	{
		FSlateApplication::Get().RequestDestroyWindow(GetMainWindow());
	}

private:
	TSharedPtr<SWidgetMock<T>> SiblingWidget;
	TSharedPtr<SWidgetMock<SVerticalBox>> PanelWidget;
};

bool FFocusTest::RunTest(const FString& Parameters)
{
	FSlateTestFocus<SButton> MyTest = FSlateTestFocus<SButton>();

	MyTest.Setup(this);
	MyTest.Run(this);
	MyTest.Validate(this);
	MyTest.TearDown(this);
	return true;
}
}
#endif