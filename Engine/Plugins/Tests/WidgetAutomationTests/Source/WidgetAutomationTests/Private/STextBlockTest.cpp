// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "SlateTestBase.h"
#include "SWidgetMock.h"

#include "Widgets/Text/STextBlock.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"

#include "Dom/JsonObject.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::SlateWidgetAutomationTest
{
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTextBlockTest, "Slate.STextBlockTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

//@TODO Should we create a new class per property? We are going to need at least different RunTest and ValidateTest functions for each property of textblock.
class FSlateTestTextBlock : public FSlateTestBase<STextBlock>
{

public:

	FSlateTestTextBlock(FString InTestName) : FSlateTestBase<STextBlock>(MoveTemp(InTestName)) {};

	void Setup(FAutomationTestBase* TestObj) override
	{
		// 1. In the main window, insert a panel widget containing the TextBlock widget being tested.
		TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox);
		VerticalBox->AddSlot()[GetSubjectWidget()];
		GetMainWindow()->SetContent(VerticalBox);

		// 2. Initialize the textbox with a sample text.
		FString TestText = "ExampleText";
		GetSubjectWidget()->SetText(FText::FromString(TestText));

		// 3. Enable fast update on the main window so that we can test global invalidation on its content.
		GetMainWindow()->SetAllowFastUpdate(true);

		// 4. Erase any side effect that we might have caused as a result of setting up the test.
		// This must be delayed to the next frame because the side effects of SetAllowFastUpdate will appear next frame.
		TestObj->AddCommand(new FDelayedFunctionLatentCommand([this, TestObj]()
			{
				GetSubjectWidget()->ClearFunctionRecords();

				// 5. Load the JSON object for this test. The data will be used as the ground truth of the test.
				// It is important to call this function after ClearFunctionRecords(), otherwise we may end up clearing the errors we faced during file I/O.
				GetSubjectWidget()->FindOrCreateTestJSONFile(GetTestName());
			}, 0.1f));
	}

	void Run(FAutomationTestBase* TestObj) override
	{
		GetSubjectWidget()->SetColorAndOpacity(FSlateColor(FLinearColor(0.5f,0.4f,0.3f,0.9f)));
	}

	void Validate(FAutomationTestBase* TestObj) override
	{
		// Ensure the JSON file for the test exists and is loaded. We don't want to continue verifications without it.
		// This should be the first thing we validate in the tests that use a JSON file.
		if (!JSONValidation(TestObj, "ColorAndOpacity"))
		{
			return;
		}

		// generic tests for property change.
		ValidatePropertyChange<FSlateColor>(TestObj, &STextBlock::GetColorAndOpacity, "ColorAndOpacity", FSlateColor(FLinearColor(0.5f, 0.4f, 0.3f, 0.9f)), EInvalidateWidgetReason::Layout|EInvalidateWidgetReason::Paint);

		// Add any non-generic tests here.

		TestObj->AddCommand(new FDelayedFunctionLatentCommand([this, TestObj]()
			{
				if (!GetMainWindow()->Advanced_IsInvalidationRoot())
				{
					// Confirm that the payload tint calculated in OnPaint matches the color we set in the test.
					if (TSharedPtr<FJsonObject> JsonObject = GetSubjectWidget()->GetTestJSONObject())
					{
						FString PayloadTint = JsonObject->GetObjectField(TEXT("ShapedTextPayload"))->GetStringField(TEXT("Tint"));
						TestObj->AddErrorIfFalse(PayloadTint == FLinearColor(0.5f, 0.4f, 0.3f, 0.9f).ToString(), FString::Printf(TEXT("In %s: The tint of the payload does not match the passed color. Expected: %s, Actual: %s"), GetGeneratedTypeName<STextBlock>(), *FLinearColor(0.5f, 0.4f, 0.3f, 0.9f).ToString(), *PayloadTint));
					}
				}
			}, 0.1f));
	}

	void TearDown(FAutomationTestBase* TestObj)
	{
		if (GetSubjectWidget()->GetTestJSONObjectToWrite().IsValid() && !FSlateTestHelper::WriteJSONToTxt(GetSubjectWidget()->GetTestJSONObjectToWrite().ToSharedRef(), GetTestName()))
		{
			TestObj->AddError(FString::Printf(TEXT("Failed to write the ground truth JSON file for the test %s."), *GetTestName()));
		}
		FSlateApplication::Get().RequestDestroyWindow(GetMainWindow());
	}
};

bool FTextBlockTest::RunTest(const FString& Parameters)
{
	TSharedPtr<FSlateTestTextBlock> MyTest = MakeShared<FSlateTestTextBlock>(TEXT("SlateTestTextBlock_ColorAndOpacity"));

	MyTest->Setup(this);

	// Wait for the next frame so that SetupTest has finished.
	// Whether we delay at each of these steps or not depends on the nature of the test. 
	// But once a step has been delayed, all the others must be too.
	AddCommand(new FDelayedFunctionLatentCommand([this, MyTest]()
		{
			MyTest->Run(this);
			MyTest->Validate(this);

			// Wait for ValidateTest to finish.
			this->AddCommand(new FDelayedFunctionLatentCommand([this, MyTest]()
				{
					MyTest->TearDown(this);
				}, 0.1f));

		}, 0.1f));

	return true;
}
}
#endif
