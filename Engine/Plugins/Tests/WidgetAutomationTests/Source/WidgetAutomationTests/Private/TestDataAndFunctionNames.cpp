// Copyright Epic Games, Inc. All Rights Reserved.
#include "TestDataAndFunctionNames.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::SlateWidgetAutomationTest
{
	namespace FTestIONames
	{
		const FLazyName NAME_FailedToLoadJSON = "FailedToLoadJSON";
		const FLazyName NAME_TestFileNotFound = "TestFileNotFound";
		const FLazyName NAME_BadTestFile = "BadTestFile";
	};

	namespace FTestFunctionNames
	{
		const FLazyName NAME_OnPaint = "OnPaint";
		const FLazyName NAME_ComputeDesiredSize = "ComputeDesiredSize";
		const FLazyName NAME_OnFocusChanging = "OnFocusChanging";
		const FLazyName NAME_OnFocusLost = "OnFocusLost";
		const FLazyName NAME_OnFocusReceived = "OnFocusReceived";
	};
}
#endif