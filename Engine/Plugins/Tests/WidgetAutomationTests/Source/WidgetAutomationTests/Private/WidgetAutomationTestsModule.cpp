// Copyright Epic Games, Inc. All Rights Reserved.

#include "IWidgetAutomationTestsModule.h"

class FWidgetAutomationTests : public IWidgetAutomationTests
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE(IWidgetAutomationTests, WidgetAutomationTests)

void FWidgetAutomationTests::StartupModule()
{
	// This code will execute after your module is loaded into memory (but after global variables are initialized, of course.)
}

void FWidgetAutomationTests::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}