// Copyright Epic Games, Inc. All Rights Reserved.
#include "WidgetMockNonTemplate.h"
#include "Dom/JsonObject.h"
#include "Misc/GeneratedTypeName.h"
#include "Rendering/DrawElements.h"
#include "Rendering/DrawElementPayloads.h"
#include "Serialization/JsonReader.h"
#include "SlateTestHelper.h"
#include "TestDataAndFunctionNames.h"
#include "Widgets/InvalidateWidgetReason.h"
#include "Widgets/SWindow.h"
#include "Misc/AutomationTest.h"
#include "Layout/SlateRect.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::SlateWidgetAutomationTest
{

void FWidgetMockNonTemplate::ClearFunctionRecords()
{
	FunctionCalls.Empty();
	DataValidations.Empty();
}

const FString& FWidgetMockNonTemplate::GetTestName() const
{
	return TestName;
}

TSharedPtr<FJsonObject> FWidgetMockNonTemplate::GetTestJSONObject() const
{
	return TestJsonObject;
}

TSharedPtr<FJsonObject> FWidgetMockNonTemplate::GetTestJSONObjectToWrite() const
{
	return WriteJsonObject;
}

void FWidgetMockNonTemplate::FindOrCreateTestJSONFile(const FString& InTestName)
{
	//@TODO Support having a separate JSON Object in the JSON test file for each mock widget. Some tests might need data from multiple mock widgets present in the test.
	// To do this, we can assign an ID to each widget when we create it and use that ID to identify the widget. Naturally, this ID must be the same for all runs of the test.

	TestName = InTestName;
	if (FPaths::FileExists(FSlateTestHelper::GetFullPath(TestName)))
	{
		TestJsonObject = FSlateTestHelper::LoadJSONFromTxt(TestName);
		WriteJsonObject = nullptr;
		if (!TestJsonObject.IsValid())
		{
			DataValidations.Add(FTestIONames::NAME_FailedToLoadJSON, true);
		}
	}
	else
	{
		DataValidations.Add(FTestIONames::NAME_TestFileNotFound, true);
		TestJsonObject = nullptr;
		WriteJsonObject = MakeShared<FJsonObject>();
	}
}

bool FWidgetMockNonTemplate::IsKeyInFunctionCalls(const FName& FuncName) const
{
	return FunctionCalls.Contains(FuncName);
}

int32 FWidgetMockNonTemplate::GetValueInFunctionCalls(const FName& FuncName) const
{
	return FunctionCalls.Contains(FuncName) ? FunctionCalls[FuncName] : 0;
}

bool FWidgetMockNonTemplate::IsKeyInDataValidations(const FName& FuncName) const
{
	return DataValidations.Contains(FuncName);
}

bool FWidgetMockNonTemplate::GetValueInDataValidations(const FName& FuncName) const
{
	return DataValidations[FuncName];
}

void FWidgetMockNonTemplate::IncrementCall(const FName& FuncName) const
{
	int32& FunctionCallNum = FunctionCalls.FindOrAdd(FuncName);
	FunctionCallNum += 1;
}

void FWidgetMockNonTemplate::OnPaintDataValidation(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	// Check if we have loaded the test JSON file or we need to write it.
	if (TestJsonObject.IsValid())
	{
		// Validate the parameters AllottedGeometry and InWidgetStyle.
		if (TestJsonObject->GetStringField(TEXT("AllottedGeometry")) == AllottedGeometry.ToString()
			&& TestJsonObject->GetObjectField(TEXT("WidgetStyle"))->GetStringField(TEXT("ColorAndOpacityTint")) == InWidgetStyle.GetColorAndOpacityTint().ToString()
			&& TestJsonObject->GetObjectField(TEXT("WidgetStyle"))->GetStringField(TEXT("ForegroundColor")) == InWidgetStyle.GetForegroundColor().ToString()
			&& TestJsonObject->GetObjectField(TEXT("WidgetStyle"))->GetStringField(TEXT("SubduedForeground")) == InWidgetStyle.GetSubduedForegroundColor().ToString())
		{
			// Validate the payloads in OutDrawElements.
			for (FSlateCachedElementList* CachedElementList : OutDrawElements.GetCurrentCachedElementWithNewData())
			{
				if (CachedElementList != nullptr)
				{
					//@TODO Support all element types and create their own JSON conversion function.
					for (const FSlateShapedTextElement& DrawElements : CachedElementList->DrawElements.Get<(uint8)EElementType::ET_ShapedText>())
					{
						if (TestJsonObject->GetObjectField(TEXT("ShapedTextElement"))->GetStringField(TEXT("Tint")) == DrawElements.GetTint().ToString()
							&& TestJsonObject->GetObjectField(TEXT("ShapedTextElement"))->GetStringField(TEXT("OutlineTint")) == DrawElements.GetOutlineTint().ToString())
						{
							if (!DataValidations.Contains(FTestFunctionNames::NAME_OnPaint))
							{
								DataValidations.Add(FTestFunctionNames::NAME_OnPaint, true);
							}
						}
						else
						{
							DataValidations.Add(FTestFunctionNames::NAME_OnPaint, false);
						}
					}
				}
			}
		}

		else
		{
			DataValidations.Add(FTestFunctionNames::NAME_OnPaint, false);
		}
	}
	else if (WriteJsonObject.IsValid())
	{
		FSlateTestHelper::ToJson(AllottedGeometry, WriteJsonObject);
		FSlateTestHelper::ToJson(InWidgetStyle, WriteJsonObject);
		for (FSlateCachedElementList* CachedElementList : OutDrawElements.GetCurrentCachedElementWithNewData())
		{
			if (CachedElementList != nullptr)
			{
				//@TODO Support all element types and create their own JSON conversion function.
				for (const FSlateShapedTextElement& DrawElements : CachedElementList->DrawElements.Get<(uint8)EElementType::ET_ShapedText>())
				{
					FSlateTestHelper::ToJson(DrawElements, WriteJsonObject);
				}
			}
		}
	}
	else
	{
		DataValidations.Add(FTestIONames::NAME_BadTestFile, true);
	}
}

void FWidgetMockNonTemplate::ComputeDesiredSizeDataValidation(float LayoutScaleMultiplier, FVector2D RealComputedSize) const
{
	// Check if we have loaded the test JSON file or we need to write it.
	if (TestJsonObject.IsValid())
	{
		// Validate the computed size in JSON against the size we just computed.
		if (TestJsonObject->GetNumberField(TEXT("ComputeDesiredSizeX")) == RealComputedSize.X && TestJsonObject->GetNumberField(TEXT("ComputeDesiredSizeY")) == RealComputedSize.Y)
		{
			DataValidations.Add(FTestFunctionNames::NAME_ComputeDesiredSize, true);

		}
		else
		{
			DataValidations.Add(FTestFunctionNames::NAME_ComputeDesiredSize, false);
		}
	}
	else if (WriteJsonObject.IsValid())
	{
		WriteJsonObject->SetNumberField(TEXT("ComputeDesiredSizeX"), RealComputedSize.X);
		WriteJsonObject->SetNumberField(TEXT("ComputeDesiredSizeY"), RealComputedSize.Y);
	}
	// If we haven't loaded the JSON file and haven't initialized a JSON object to write to, this is an error and we can't continue the test.
	else
	{
		DataValidations.Add(FTestIONames::NAME_BadTestFile, true);
	}
}

}
#endif