// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#if WITH_DEV_AUTOMATION_TESTS
class FJsonObject;
struct FGeometry;
class FSlateRect;
class FSlateWindowElementList;
class FWidgetStyle;

namespace UE::SlateWidgetAutomationTest
{

// Do NOT create instances of this class. It is used for refactoring the template class SWidgetMock. Use SWidgetMock for your tests.
class FWidgetMockNonTemplate
{
public:
	void ClearFunctionRecords();

	const FString& GetTestName() const;
	TSharedPtr<FJsonObject> GetTestJSONObject() const;
	TSharedPtr<FJsonObject> GetTestJSONObjectToWrite() const;
	void FindOrCreateTestJSONFile(const FString& InTestName);

	bool IsKeyInFunctionCalls(const FName& FuncName) const;

	// Return the number of times we called the function with the name FuncName.
	int32 GetValueInFunctionCalls(const FName& FuncName) const;

	bool IsKeyInDataValidations(const FName& FuncName) const;
	bool GetValueInDataValidations(const FName& FuncName) const;

	// Increase (by 1) the number of times we called the function with the name FuncName.
	void IncrementCall(const FName& FuncName) const;

	void OnPaintDataValidation(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const;

	void ComputeDesiredSizeDataValidation(float LayoutScaleMultiplier, FVector2D RealComputedSize) const;

private:
	// Map to store function calls and the number of times they were called.
	// Ex. (OnComputeSize, 3) means OnComputeSize was called 3 times.
	mutable TMap<FName, int32> FunctionCalls;

	// Map to store the result of the verification of parameters and outputs of a function.
	// Ex. (OnComputeSize, true) means the parameters and output of the function OnComputeSize were correct according to the test JSON baseline.
	mutable TMap<FName, bool> DataValidations;

	// This will be incorporated in the name of the JSON baseline file.
	// Make sure it is unique and that it is set in all important widgets. 
	FString TestName;

	// The JSON object loaded from the test JSON file on disk.
	TSharedPtr<FJsonObject> TestJsonObject;

	// JSON object that collects and holds all data that needs to be written to the JSON file.
	// If we initially find a JSON file for the test, this will remain null.
	TSharedPtr<FJsonObject> WriteJsonObject;
};

}
#endif