// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "TestDataAndFunctionNames.h"
#include "WidgetMockNonTemplate.h"
#include "Widgets/SWidget.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::SlateWidgetAutomationTest
{
// All widgets in a test should inherit from this class.
template<class T>
class SWidgetMock : public T, public FWidgetMockNonTemplate
{
	static_assert(TIsDerivedFrom<T, SWidget>::IsDerived, "The mock widget must be a subclass of SWidget.");
public:

	virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override
	{
		T::OnFocusReceived(MyGeometry, InFocusEvent);
		IncrementCall(FTestFunctionNames::NAME_OnFocusReceived);
		return FReply::Unhandled();
	}

	virtual void OnFocusLost(const FFocusEvent& InFocusEvent) override
	{
		T::OnFocusLost(InFocusEvent);
		IncrementCall(FTestFunctionNames::NAME_OnFocusLost);
	}

	virtual void OnFocusChanging(const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent) override
	{
		T::OnFocusChanging(PreviousFocusPath, NewWidgetPath, InFocusEvent);
		IncrementCall(FTestFunctionNames::NAME_OnFocusChanging);
	}

protected:
	virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override
	{
		IncrementCall(FTestFunctionNames::NAME_ComputeDesiredSize);
		FVector2D RealComputedSize = T::ComputeDesiredSize(LayoutScaleMultiplier);

		ComputeDesiredSizeDataValidation(LayoutScaleMultiplier, RealComputedSize);
		return RealComputedSize;
	}

private:
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		IncrementCall(FTestFunctionNames::NAME_OnPaint);
		int32 RealLayer = T::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

		this->OnPaintDataValidation(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
		return RealLayer;
	}
};
}
#endif