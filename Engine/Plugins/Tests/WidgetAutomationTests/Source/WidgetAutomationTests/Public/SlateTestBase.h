// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/AutomationTest.h"
#include "Misc/GeneratedTypeName.h"
#include "SlateTestHelper.h"
#include "TestDataAndFunctionNames.h"
#include "Widgets/InvalidateWidgetReason.h"
#include "Widgets/SWindow.h"

#if WITH_DEV_AUTOMATION_TESTS

class SWidget;

namespace UE::SlateWidgetAutomationTest
{
template<class T>
class SWidgetMock;

// Base interface that all slate widget tests should inherit from
template <class T>
class FSlateTestBase
{
	static_assert(TIsDerivedFrom<T, SWidget>::IsDerived, "The template class of the test must be a subclass of SWidget.");

public:

	FSlateTestBase() : FSlateTestBase(TEXT(""))
	{}

	FSlateTestBase(FString InTestName) : 
		MainWindow(SNew(SWindow)
			.ClientSize(FVector2D(1000.f, 700.f))
			.SupportsMinimize(false)),
		SubjectWidget(CreateWidget()),
		TestName(MoveTemp(InTestName))
	{
		FSlateApplication::Get().AddWindow(MainWindow);
	}

	virtual void Setup(FAutomationTestBase* TestObj) = 0;

	virtual void Run(FAutomationTestBase* TestObj) = 0;

	virtual void Validate(FAutomationTestBase* TestObj) = 0;

	virtual void TearDown(FAutomationTestBase* TestObj) = 0;

	TSharedRef<SWindow> GetMainWindow() const
	{
		return MainWindow;
	}

	TSharedRef<SWidgetMock<T>> GetSubjectWidget() const
	{
		return SubjectWidget;
	}

	TSharedRef< SWidgetMock<T>> CreateWidget() const
	{
		return SNew(SWidgetMock<T>);
	}

	const FString& GetTestName() const
	{
		return TestName;
	}

	// This function consists of generic verifications that we want to do for every property change in a SWidget.
	template< typename PropertyType>
	void ValidatePropertyChange(FAutomationTestBase* TestObj, typename TMemFunPtrType<true, T, PropertyType()>::Type Getter, const FString& PropertyName, PropertyType TestValue, EInvalidateWidgetReason InvalidateReason)
	{
		// 1. Verify that the property getter returns the value we set. 
		TestObj->AddErrorIfFalse((SubjectWidget.Get().*Getter)() == TestValue, FString::Printf(TEXT("In %s: The property %s was not set."), GetGeneratedTypeName<T>(), *PropertyName));

		// 2. Ensure global invalidation is on. Otherwise we will skip checking if the widget has a valid handle. 
		if (!MainWindow->Advanced_IsInvalidationRoot())
		{
			TestObj->AddError(FString::Printf(TEXT("In %s: Slate.EnableGlobalInvalidation is disabled. Please enable it and run the test again."), GetGeneratedTypeName<T>()));
		}
		else
		{
			TestObj->AddErrorIfFalse(SubjectWidget->GetProxyHandle().IsValid(SubjectWidget.Get()), FString::Printf(TEXT("In %s: The widget doesn't have a valid Invalidation Root. Make sure the widget is added to a window in the previous frame, and the window has fast update enabled."), GetGeneratedTypeName<T>()));

			// 3. In the next frame, verify if requested by the invalidation reason:
			// - ComputeDesiredSize was called on the widget whose property has changed
			// - ComputeDesiredSize returns the correct size.
			// - OnPaint was called on the widget whose property has changed
			// - OnPaint has the correct values for Allotted Geometry and WidgetStyle
			TestObj->AddCommand(new FDelayedFunctionLatentCommand([this, WidgetRef = GetSubjectWidget(), TestObj, InvalidateReason, PropertyName, MyWindow = MainWindow]()
				{
					// We have to do this JSON check here because we attempt to read/write the JSON file in this frame for the first time.
					if (WidgetRef->IsKeyInDataValidations(FTestIONames::NAME_BadTestFile))
					{
						FString FileLocation = FSlateTestHelper::GetFullPath(TestName);
						TestObj->AddError(FString::Printf(TEXT("In property %s in %s. The ground truth JSON file '%s' for this test was not found and could not be created. Did you forget to call FindOrCreateTestJSONFile in the setup of your test?"), *PropertyName, GetGeneratedTypeName<T>(), *FileLocation));
					}
					else
					{
						if (EnumHasAnyFlags(InvalidateReason, EInvalidateWidgetReason::Layout))
						{
							TestObj->AddErrorIfFalse(WidgetRef->GetValueInFunctionCalls(FTestFunctionNames::NAME_ComputeDesiredSize) == 1, FString::Printf(TEXT("In %s: ComputeDesiredSize was not called after the property %s changed."), GetGeneratedTypeName<T>(), *PropertyName));
							TestObj->AddErrorIfFalse(WidgetRef->IsKeyInDataValidations(FTestFunctionNames::NAME_ComputeDesiredSize) && WidgetRef->GetValueInDataValidations(FTestFunctionNames::NAME_ComputeDesiredSize), FString::Printf(TEXT("In %s: The computed desired size is wrong or does not exist after property %s changed."), GetGeneratedTypeName<T>(), *PropertyName));
						}

						if (EnumHasAnyFlags(InvalidateReason, EInvalidateWidgetReason::Paint))
						{
							TestObj->AddErrorIfFalse(WidgetRef->GetValueInFunctionCalls(FTestFunctionNames::NAME_OnPaint) == 1, FString::Printf(TEXT("In %s: OnPaint was not called after the property %s changed in widget. Make sure the widget is being invalidated."), GetGeneratedTypeName<T>(), *PropertyName));
							TestObj->AddErrorIfFalse(WidgetRef->IsKeyInDataValidations(FTestFunctionNames::NAME_OnPaint) && WidgetRef->GetValueInDataValidations(FTestFunctionNames::NAME_OnPaint), FString::Printf(TEXT("In %s: The list of elements to draw is wrong or does not exist."), GetGeneratedTypeName<T>()));
						}
					}

				}, 0.1f));
		}
	};

	virtual ~FSlateTestBase() = default;

	bool JSONValidation(FAutomationTestBase* TestObj, const FString& PropertyName)
	{
		if (SubjectWidget->IsKeyInDataValidations(FTestIONames::NAME_TestFileNotFound))
		{
			FString FileLocation = FSlateTestHelper::GetFullPath(TestName);
			TestObj->AddError(FString::Printf(TEXT("Skipping the test for the property %s in %s. The ground truth JSON file for this test was not found looking under path: %s.\n This is expected only if this test is being run for the first time after creation or modification. If this is the case, please run the test again."), *PropertyName, GetGeneratedTypeName<T>(), *FileLocation));
			return false;
		}
		if (SubjectWidget->IsKeyInDataValidations(FTestIONames::NAME_FailedToLoadJSON))
		{
			TestObj->AddError(FString::Printf(TEXT("Skipping the test for the property %s in %s. The ground truth JSON file for this test was found but could not be loaded."), *PropertyName, GetGeneratedTypeName<T>()));
			return false;
		}
		return true;
	}

private:
	TSharedRef<SWindow> MainWindow;
	TSharedRef<SWidgetMock<T>> SubjectWidget; 
	FString TestName;
};
}

#endif