// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Dom/JsonObject.h"
#include "Styling/WidgetStyle.h"
#include "Layout/Geometry.h"

#if WITH_DEV_AUTOMATION_TESTS

class JsonObject;
struct FSlateShapedTextElement;

namespace UE::SlateWidgetAutomationTest
{
// A collection of helper functions used for slate tests.
class FSlateTestHelper
{
public:	
	// Write ObjToWrite to the JSON file with name TestName.
	static bool WriteJSONToTxt(TSharedRef<FJsonObject> ObjToWrite, const FString& TestName);

	// Load and return the JSON object in JSON file TestName.
	static TSharedPtr<FJsonObject> LoadJSONFromTxt(const FString& TestName);

	static FString GetFullPath(FStringView TestName);
	
	// JSON conversion functions.
	static void ToJson(const FGeometry& MyGeometry, TSharedPtr<FJsonObject> MyJsonObject);
	static void ToJson(const FWidgetStyle& InWidgetStyle, TSharedPtr<FJsonObject> MyJsonObject);
	static void ToJson(const FSlateShapedTextElement& ShapedTextElement, TSharedPtr<FJsonObject> MyJsonObject);
};
}
#endif