// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "UObject/NameTypes.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::SlateWidgetAutomationTest
{
	namespace FTestIONames
	{
		 extern const FLazyName NAME_FailedToLoadJSON;
		 extern const FLazyName NAME_TestFileNotFound;
		 extern const FLazyName NAME_BadTestFile;
	};

	namespace FTestFunctionNames
	{
		extern const FLazyName NAME_OnPaint;
		extern const FLazyName NAME_ComputeDesiredSize;
		extern const FLazyName NAME_OnFocusChanging;
		extern const FLazyName NAME_OnFocusLost;
		extern const FLazyName NAME_OnFocusReceived;
	};
}
#endif