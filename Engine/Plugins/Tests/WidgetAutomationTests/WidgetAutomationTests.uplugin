{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "WidgetAutomationTests",
	"Description": "",
	"Category": "Testing",
	"CreatedBy": "Epic Games, Inc",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "WidgetAutomationTests",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "EditorTests",
			"Enabled": true
		}
	]
}